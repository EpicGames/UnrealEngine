// Copyright Epic Games, Inc. All Rights Reserved.

#include "RawMesh.h"
#include "Serialization/BufferWriter.h"
#include "UObject/Object.h"
#include "Misc/SecureHash.h"
#include "Modules/ModuleManager.h"
#include "Serialization/BulkDataReader.h"
#include "Misc/ScopeRWLock.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, RawMesh);

/*------------------------------------------------------------------------------
	FRawMesh
------------------------------------------------------------------------------*/

void FRawMesh::Empty()
{
	FaceMaterialIndices.Empty();
	FaceSmoothingMasks.Empty();
	VertexPositions.Empty();
	WedgeIndices.Empty();
	WedgeTangentX.Empty();
	WedgeTangentY.Empty();
	WedgeTangentZ.Empty();
	WedgeColors.Empty();
	for (int32 i = 0; i < MAX_MESH_TEXTURE_COORDS; ++i)
	{
		WedgeTexCoords[i].Empty();
	}
}

template <typename ArrayType>
bool ValidateArraySize(ArrayType const& Array, int32 ExpectedSize)
{
	return Array.Num() == 0 || Array.Num() == ExpectedSize;
}

bool FRawMesh::IsValid() const
{
	int32 NumVertices = VertexPositions.Num();
	int32 NumWedges = WedgeIndices.Num();
	int32 NumFaces = NumWedges / 3;

	bool bValid = NumVertices > 0
		&& NumWedges > 0
		&& NumFaces > 0
		&& (NumWedges / 3) == NumFaces
		&& ValidateArraySize(FaceMaterialIndices, NumFaces)
		&& ValidateArraySize(FaceSmoothingMasks, NumFaces)
		&& ValidateArraySize(WedgeTangentX, NumWedges)
		&& ValidateArraySize(WedgeTangentY, NumWedges)
		&& ValidateArraySize(WedgeTangentZ, NumWedges)
		&& ValidateArraySize(WedgeColors, NumWedges)
		// All meshes must have a valid texture coordinate.
		&& WedgeTexCoords[0].Num() == NumWedges; 

	for (int32 TexCoordIndex = 1; TexCoordIndex < MAX_MESH_TEXTURE_COORDS; ++TexCoordIndex)
	{
		bValid = bValid && ValidateArraySize(WedgeTexCoords[TexCoordIndex], NumWedges);
	}

	int32 WedgeIndex = 0;
	while (bValid && WedgeIndex < NumWedges)
	{
		bValid = bValid && ( WedgeIndices[WedgeIndex] < (uint32)NumVertices );
		WedgeIndex++;
	}

	return bValid;
}

bool FRawMesh::IsValidOrFixable() const
{
	int32 NumVertices = VertexPositions.Num();
	int32 NumWedges = WedgeIndices.Num();
	int32 NumFaces = NumWedges / 3;
	int32 NumTexCoords = WedgeTexCoords[0].Num();
	int32 NumFaceSmoothingMasks = FaceSmoothingMasks.Num();
	int32 NumFaceMaterialIndices = FaceMaterialIndices.Num();

	bool bValidOrFixable = NumVertices > 0
		&& NumWedges > 0
		&& NumFaces > 0
		&& (NumWedges / 3) == NumFaces
		&& NumFaceMaterialIndices == NumFaces
		&& NumFaceSmoothingMasks == NumFaces
		&& ValidateArraySize(WedgeColors, NumWedges)
		// All meshes must have a valid texture coordinate.
		&& NumTexCoords == NumWedges; 

	for (int32 TexCoordIndex = 1; TexCoordIndex < MAX_MESH_TEXTURE_COORDS; ++TexCoordIndex)
	{
		bValidOrFixable = bValidOrFixable && ValidateArraySize(WedgeTexCoords[TexCoordIndex], NumWedges);
	}

	int32 WedgeIndex = 0;
	while (bValidOrFixable && WedgeIndex < NumWedges)
	{
		bValidOrFixable = bValidOrFixable && ( WedgeIndices[WedgeIndex] < (uint32)NumVertices );
		WedgeIndex++;
	}

	return bValidOrFixable;
}

void FRawMesh::CompactMaterialIndices()
{
	MaterialIndexToImportIndex.Reset();
	if (IsValidOrFixable())
	{
		// Count the number of triangles per section.
		TArray<int32, TInlineAllocator<8> > NumTrianglesPerSection;
		int32 NumFaces = FaceMaterialIndices.Num();
		for (int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
		{
			int32 MaterialIndex = FaceMaterialIndices[FaceIndex];
			if (MaterialIndex >= NumTrianglesPerSection.Num())
			{
				NumTrianglesPerSection.AddZeroed(MaterialIndex - NumTrianglesPerSection.Num() + 1);
			}
			if (MaterialIndex >= 0)
			{
				NumTrianglesPerSection[MaterialIndex]++;
			}
		}

		// Identify non-zero sections and assign new materials.
		TArray<int32, TInlineAllocator<8> > ImportIndexToMaterialIndex;
		for (int32 SectionIndex = 0; SectionIndex < NumTrianglesPerSection.Num(); ++SectionIndex)
		{
			int32 NewMaterialIndex = INDEX_NONE;
			if (NumTrianglesPerSection[SectionIndex] > 0)
			{
				NewMaterialIndex = MaterialIndexToImportIndex.Add(SectionIndex);
			}
			ImportIndexToMaterialIndex.Add(NewMaterialIndex);
		}

		// If some sections will be removed, remap material indices for each face.
		if (MaterialIndexToImportIndex.Num() != ImportIndexToMaterialIndex.Num())
		{
			for (int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
			{
				int32 MaterialIndex = FaceMaterialIndices[FaceIndex];
				FaceMaterialIndices[FaceIndex] = ImportIndexToMaterialIndex[MaterialIndex];
			}
		}
		else
		{
			MaterialIndexToImportIndex.Reset();
		}
	}
}

/*------------------------------------------------------------------------------
	FRawMeshBulkData
------------------------------------------------------------------------------*/

FRawMeshBulkData::FRawMeshBulkData()
	: bGuidIsHash(false)
{
}

/**
 * Serialization of raw meshes uses its own versioning scheme because it is
 * stored in bulk data.
 */
enum
{
	// Engine raw mesh version:
	RAW_MESH_VER_INITIAL = 0,
	RAW_MESH_VER_REMOVE_ZERO_TRIANGLE_SECTIONS,
	// Add new raw mesh versions here.

	RAW_MESH_VER_PLUS_ONE,
	RAW_MESH_VER = RAW_MESH_VER_PLUS_ONE - 1,
	
	// Licensee raw mesh version:
	RAW_MESH_LIC_VER_INITIAL = 0,
	// Licensees add new raw mesh versions here.

	RAW_MESH_LIC_VER_PLUS_ONE,
	RAW_MESH_LIC_VER = RAW_MESH_LIC_VER_PLUS_ONE - 1
};

FArchive& operator<<(FArchive& Ar, FRawMesh& RawMesh)
{
	int32 Version = RAW_MESH_VER;
	int32 LicenseeVersion = RAW_MESH_LIC_VER;
	Ar << Version;
	Ar << LicenseeVersion;

	/**
	 * Serialization should use the raw mesh version not the archive version.
	 * Additionally, stick to serializing basic types and arrays of basic types.
	 */

	Ar << RawMesh.FaceMaterialIndices;
	Ar << RawMesh.FaceSmoothingMasks;
	Ar << RawMesh.VertexPositions;
	Ar << RawMesh.WedgeIndices;
	Ar << RawMesh.WedgeTangentX;
	Ar << RawMesh.WedgeTangentY;
	Ar << RawMesh.WedgeTangentZ;
	for (int32 i = 0; i < MAX_MESH_TEXTURE_COORDS; ++i)
	{
		Ar << RawMesh.WedgeTexCoords[i];
	}
	Ar << RawMesh.WedgeColors;

	if (Version < RAW_MESH_VER_REMOVE_ZERO_TRIANGLE_SECTIONS)
	{
		RawMesh.CompactMaterialIndices();
	}
	else
	{
		Ar << RawMesh.MaterialIndexToImportIndex;
	}

	return Ar;
}

void FRawMeshBulkData::Serialize(FArchive& Ar, UObject* Owner)
{
	BulkData.Serialize(Ar, Owner);
	Ar << Guid;
	Ar << bGuidIsHash;
}

int64 GetRawMeshSerializedDataSize(const FRawMesh& InMesh)
{
	int64 NumBytes = 0;
	NumBytes += sizeof(int32);
	NumBytes += sizeof(int32);
	NumBytes += sizeof(int32) + InMesh.FaceMaterialIndices.Num() * InMesh.FaceMaterialIndices.GetTypeSize();
	NumBytes += sizeof(int32) + InMesh.FaceSmoothingMasks.Num() * InMesh.FaceSmoothingMasks.GetTypeSize();
	NumBytes += sizeof(int32) + InMesh.VertexPositions.Num() * InMesh.VertexPositions.GetTypeSize();
	NumBytes += sizeof(int32) + InMesh.WedgeIndices.Num() * InMesh.WedgeIndices.GetTypeSize();
	NumBytes += sizeof(int32) + InMesh.WedgeTangentX.Num() * InMesh.WedgeTangentX.GetTypeSize();
	NumBytes += sizeof(int32) + InMesh.WedgeTangentY.Num() * InMesh.WedgeTangentY.GetTypeSize();
	NumBytes += sizeof(int32) + InMesh.WedgeTangentZ.Num() * InMesh.WedgeTangentZ.GetTypeSize();
	for (int32 i = 0; i < MAX_MESH_TEXTURE_COORDS; ++i)
	{
		NumBytes += sizeof(int32) + InMesh.WedgeTexCoords[i].Num() * InMesh.WedgeTexCoords[i].GetTypeSize();
	}
	NumBytes += sizeof(int32) + InMesh.WedgeColors.Num() * InMesh.WedgeColors.GetTypeSize();
	NumBytes += sizeof(int32) + InMesh.MaterialIndexToImportIndex.Num() * InMesh.MaterialIndexToImportIndex.GetTypeSize();
	return NumBytes;
}

void FRawMeshBulkData::SaveRawMesh(FRawMesh& InMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRawMeshBulkData::SaveRawMesh);

	int64 NumBytes = GetRawMeshSerializedDataSize(InMesh);
	BulkData.Lock(LOCK_READ_WRITE);
	uint8* Dest = (uint8*)BulkData.Realloc(NumBytes);
	FBufferWriter Ar(Dest, NumBytes);
	Ar.SetIsPersistent(true);
	Ar << InMesh;
	check(Ar.AtEnd());
	check(Dest == Ar.GetWriterData());
	BulkData.Unlock();
	FPlatformMisc::CreateGuid(Guid);
}

void FRawMeshBulkData::LoadRawMesh(FRawMesh& OutMesh)
{
	OutMesh.Empty();
	if (BulkData.GetElementCount() > 0)
	{
#if WITH_EDITOR
		// A lock is required so we can safely load the raw data from multiple threads
		FRWScopeLock ScopeLock(BulkDataLock.Get(), SLT_Write);

		// This allows any thread to be able to deserialize from the RawMesh directly
		// from disk so we can unload bulk data from memory.
		bool bHasBeenLoadedFromFileReader = false;
		if (!BulkData.IsBulkDataLoaded())
		{
			// This can't be called in -game mode because we're not allowed to load bulk data outside of EDL.
			bHasBeenLoadedFromFileReader = BulkData.LoadBulkDataWithFileReader();
		}
#endif

		// This is in a scope because the FBulkDataReader need to be destroyed in order
		// to unlock the BulkData and allow UnloadBulkData to actually do its job.
		{
			const bool bIsPersistent = true;
			FBulkDataReader Ar(BulkData, bIsPersistent);
			Ar << OutMesh;
		}

#if WITH_EDITOR
		// Throw away the bulk data allocation only in the case we can safely reload it from disk
		// and if BulkData.LoadBulkDataWithFileReader() is allowed to work from any thread.
		// This saves a significant amount of memory during map loading of Nanite Meshes.
		if (bHasBeenLoadedFromFileReader)
		{
			verify(BulkData.UnloadBulkData());
		}
#endif
	}
}

FString FRawMeshBulkData::GetIdString() const
{
	FString GuidString = Guid.ToString();
	if (bGuidIsHash)
	{
		GuidString += TEXT("X");
	}
	return GuidString;
}

void FRawMeshBulkData::UseHashAsGuid(UObject* Owner)
{
	// Build the hash from the path name + the contents of the bulk data.
	FSHA1 Sha;
	TArray<TCHAR, FString::AllocatorType> OwnerName = Owner->GetPathName().GetCharArray();
	Sha.Update((uint8*)OwnerName.GetData(), OwnerName.Num() * OwnerName.GetTypeSize());
	if (BulkData.GetBulkDataSize() > 0)
	{
		uint8* Buffer = (uint8*)BulkData.Lock(LOCK_READ_ONLY);
		Sha.Update(Buffer, BulkData.GetBulkDataSize());
		BulkData.Unlock();
	}
	Sha.Final();

	// Retrieve the hash and use it to construct a pseudo-GUID. Use bGuidIsHash to distinguish from real guids.
	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	Guid = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
	bGuidIsHash = true;
}

const FByteBulkData& FRawMeshBulkData::GetBulkData() const
{
	return BulkData;
}

void FRawMeshBulkData::Empty()
{
	BulkData.RemoveBulkData();
	Guid.Invalidate();
	bGuidIsHash = false;
}