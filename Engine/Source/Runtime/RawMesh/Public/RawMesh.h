// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/Guid.h"
#include "Serialization/BulkData.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Templates/DontCopy.h"

class FArchive;

enum
{
	MAX_MESH_TEXTURE_COORDS = 8,
};

/**
 * Raw mesh data used to construct optimized runtime rendering streams.
 *
 * A note on terminology. Information is stored at various frequencies as defined here:
 *     Face - A single polygon in the mesh. Currently all code assumes this is a triangle but
 *            conceptually any polygon would do.
 *     Corner - Each face has N corners. As all faces are currently triangles, N=3.
 *     Wedge - Properties stored for each corner of each face. Index with FaceIndex * NumCorners + CornerIndex.
 *     Vertex - Properties shared by overlapping wedges of adjacent polygons. Typically these properties
 *              relate to position. Index with VertexIndices[WedgeIndex].
 *
 * Additionally, to ease in backwards compatibility all properties should use only primitive types!
 */
struct FRawMesh
{
	/** Material index. Array[FaceId] = int32 */
	TArray<int32> FaceMaterialIndices;
	/** Smoothing mask. Array[FaceId] = uint32 */
	TArray<uint32> FaceSmoothingMasks;

	/** Position in local space. Array[VertexId] = float3(x,y,z) */
	TArray<FVector3f> VertexPositions;

	/** Index of the vertex at this wedge. Array[WedgeId] = VertexId */
	TArray<uint32> WedgeIndices;
	/** Tangent, U direction. Array[WedgeId] = float3(x,y,z) */
	TArray<FVector3f>	WedgeTangentX;
	/** Tangent, V direction. Array[WedgeId] = float3(x,y,z) */
	TArray<FVector3f>	WedgeTangentY;
	/** Normal. Array[WedgeId] = float3(x,y,z) */
	TArray<FVector3f>	WedgeTangentZ;
	/** Texture coordinates. Array[UVId][WedgeId]=float2(u,v) */
	TArray<FVector2f> WedgeTexCoords[MAX_MESH_TEXTURE_COORDS];
	/** Color. Array[WedgeId]=float3(r,g,b,a) */
	TArray<FColor> WedgeColors;

	/**
	 * Map from material index -> original material index at import time. It's
	 * valid for this to be empty in which case material index == original
	 * material index.
	 */
	TArray<int32> MaterialIndexToImportIndex;

	/** Empties all data streams. */
	RAWMESH_API void Empty();

	/** 
	 * Returns true if the mesh contains valid information.
	 *  - Validates that stream sizes match.
	 *  - Validates that there is at least one texture coordinate.
	 *  - Validates that indices are valid positions in the vertex stream.
	 */
	RAWMESH_API bool IsValid() const;

	/** 
	 * Returns true if the mesh contains valid information or slightly invalid information that we can fix.
	 *  - Validates that stream sizes match.
	 *  - Validates that there is at least one texture coordinate.
	 *  - Validates that indices are valid positions in the vertex stream.
	 */
	RAWMESH_API bool IsValidOrFixable() const;

	/** Helper for getting the position of a wedge. */
	inline FVector3f GetWedgePosition(int32 WedgeIndex) const
	{
		return VertexPositions[WedgeIndices[WedgeIndex]];
	}

	/**
	 * Compacts materials by removing any that have no associated triangles.
	 * Also updates the material index map.
	 */
	RAWMESH_API void CompactMaterialIndices();

};

/**
* Add the raw mesh data to an archive
*
*/
RAWMESH_API FArchive& operator<<(FArchive& Ar, FRawMesh& RawMesh);


/**
 * Bulk data storage for raw meshes.
 */
class FRawMeshBulkData
{
#if WITH_EDITOR
	/** Protects simultaneous access to BulkData */
	TDontCopy<FRWLock> BulkDataLock;
#endif
	/** Internally store bulk data as bytes. */
	FByteBulkData BulkData;
	/** GUID associated with the data stored herein. */
	FGuid Guid;
	/** If true, the GUID is actually a hash of the contents. */
	bool bGuidIsHash;

public:
	/** Default constructor. */
	RAWMESH_API FRawMeshBulkData();

	/** Serialization. */
	RAWMESH_API void Serialize(class FArchive& Ar, class UObject* Owner);

	/** Store a new raw mesh in the bulk data. */
	RAWMESH_API void SaveRawMesh(struct FRawMesh& InMesh);

	/** Load the raw mesh from bulk data. */
	RAWMESH_API void LoadRawMesh(struct FRawMesh& OutMesh);

	/** Retrieve a string uniquely identifying the contents of this bulk data. */
	RAWMESH_API FString GetIdString() const;

	/** Uses a hash as the GUID, useful to prevent creating new GUIDs on load for legacy assets. */
	RAWMESH_API void UseHashAsGuid(class UObject* Owner);
	bool UsesHashAsGuid() const { return bGuidIsHash; }

	RAWMESH_API const FByteBulkData& GetBulkData() const;

	/** Returns true if no bulk data is available for this mesh. */
	inline bool IsEmpty() const { return BulkData.GetBulkDataSize() == 0; }

	/** Returns true if no bulk data is available for this mesh. */
	RAWMESH_API void Empty();
};
